{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Geometry Dataflow Nodes",
    "Description" : "Geometry Processing in Dataflow.",
    "Category": "Geometry",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : true,
    "IsBetaVersion" : false,
    "Installed" : false,
	"Modules": [
		{
			"Name": "GeometryDataflowNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataflowMedialSkeleton",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}
	]
}
