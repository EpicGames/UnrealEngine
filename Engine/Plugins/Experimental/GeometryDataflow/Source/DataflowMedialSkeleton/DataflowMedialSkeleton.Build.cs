// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataflowMedialSkeleton : ModuleRules
	{
        public DataflowMedialSkeleton(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"GeometryAlgorithms"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
				}
			);
		}
	}
}
