// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataflowMedialSkeletonModule.h"

#include "CoreMinimal.h"

#define LOCTEXT_NAMESPACE "DataflowMedialSkeleton"


void IGeometryDataflowMedialSkeletonPlugin::StartupModule()
{
}

void IGeometryDataflowMedialSkeletonPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IGeometryDataflowMedialSkeletonPlugin, DataflowMedialSkeleton)


#undef LOCTEXT_NAMESPACE
