// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Skeletonization/MeshMedialAxisSampling.h"

#include "DataflowMedialSkeleton.generated.h"

//~ Simple wrapper class to work with medial skeletons in dataflow
// A set of spheres and associated connectivity corresponding to a sampled medial skeleton of a target mesh
USTRUCT()
struct FDataflowMedialSkeleton
{
	GENERATED_USTRUCT_BODY()

public:
	UE::Geometry::MedialAxis::FMedialSkeleton Skeleton;
};

