// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeometryDataflowNodes : ModuleRules
	{
        public GeometryDataflowNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DataflowCore",
					"DataflowEngine",
					"GeometryCore",
					"GeometryFramework" // for UDynamicMesh
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"DynamicMesh"
				}
			);
		}
	}
}
