// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/MeshBooleanNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Operations/MeshBoolean.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshBooleanNodes)

namespace UE::Dataflow
{

	void MeshBooleanNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshBooleanDataflowNode);
	}
}


FMeshBooleanDataflowNode::FMeshBooleanDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh1);
	RegisterInputConnection(&Mesh2);
	RegisterOutputConnection(&Mesh);
}

void FMeshBooleanDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		if (const TObjectPtr<UDynamicMesh> InMesh1 = GetValue(Context, &Mesh1))
		{
			if (const TObjectPtr<UDynamicMesh> InMesh2 = GetValue(Context, &Mesh2))
			{
				const UE::Geometry::FDynamicMesh3& DynMesh1 = InMesh1->GetMeshRef();
				const UE::Geometry::FDynamicMesh3& DynMesh2 = InMesh2->GetMeshRef();

				if (DynMesh1.VertexCount() > 0 && DynMesh2.VertexCount() > 0)
				{
					// Get output
					TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
					NewMesh->Reset();
					UE::Geometry::FDynamicMesh3& ResultDynMesh = NewMesh->GetMeshRef();

					UE::Geometry::FMeshBoolean::EBooleanOp BoolOp = UE::Geometry::FMeshBoolean::EBooleanOp::Intersect;
					if (Operation == EMeshBooleanOperationEnum::Dataflow_MeshBoolean_Intersect)
					{
						BoolOp = UE::Geometry::FMeshBoolean::EBooleanOp::Intersect;
					}
					else if (Operation == EMeshBooleanOperationEnum::Dataflow_MeshBoolean_Union)
					{
						BoolOp = UE::Geometry::FMeshBoolean::EBooleanOp::Union;
					}
					else if (Operation == EMeshBooleanOperationEnum::Dataflow_MeshBoolean_Difference)
					{
						BoolOp = UE::Geometry::FMeshBoolean::EBooleanOp::Difference;
					}

					UE::Geometry::FMeshBoolean Boolean(&DynMesh1, &DynMesh2, &ResultDynMesh, BoolOp);
					Boolean.bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;
					Boolean.bPreserveOverlayUVs = (PreserveUVQualityForMesh != EDataflowMeshBooleanInputMeshEnum::None);
					Boolean.bPreserveVertexUVs = (PreserveUVQualityForMesh != EDataflowMeshBooleanInputMeshEnum::None);
					Boolean.PreserveUVsOnlyForMesh = PreserveUVQualityForMesh == EDataflowMeshBooleanInputMeshEnum::First ? 0 :
						((PreserveUVQualityForMesh == EDataflowMeshBooleanInputMeshEnum::Second) ? 1 : INDEX_NONE);
					Boolean.bWeldSharedEdges = bWeldSharedEdges;
					Boolean.bTrackAllNewEdges = false;

					Boolean.Compute();
					SetValue(Context, NewMesh, &Mesh);
					return;
				}
			}
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
}

