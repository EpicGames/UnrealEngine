// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryDataflowNodesPlugin.h"

#include "CoreMinimal.h"

#include "Dataflow/MeshBooleanNodes.h"

#define LOCTEXT_NAMESPACE "DataflowNodes"


void IGeometryDataflowNodesPlugin::StartupModule()
{
	UE::Dataflow::MeshBooleanNodes();
}

void IGeometryDataflowNodesPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IGeometryDataflowNodesPlugin, GeometryDataflowNodes)


#undef LOCTEXT_NAMESPACE
