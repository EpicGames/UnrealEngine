// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshProcessor.h"

#include "MeshBooleanNodes.generated.h"

class UDynamicMesh;
class UMaterialInterface;


UENUM(BlueprintType)
enum class EMeshBooleanOperationEnum : uint8
{
	// A union of A + B includes everything inside either A or B
	Dataflow_MeshBoolean_Union UMETA(DisplayName = "Union"),
	// An intersection of A & B includes only the points inside both A and B, i.e. trimming A by B (and vice versa)
	Dataflow_MeshBoolean_Intersect UMETA(DisplayName = "Intersect"),
	// A difference of A - B includes only the points inside A that are outside of B, i.e. subtracting B from A
	Dataflow_MeshBoolean_Difference UMETA(DisplayName = "Difference"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

// Enum to select among the input meshes for a Mesh Boolean operation
UENUM(BlueprintType)
enum class EDataflowMeshBooleanInputMeshEnum : uint8
{
	None,
	First,
	Second,
	Both
};

/**
 *
 * Compute a Mesh Boolean between Mesh1 and Mesh2
 * 
 * Supported Boolean Operations:
 *  Union (Mesh1 + Mesh2)
 *  Difference (Mesh1 - Mesh2; removing what's inside of Mesh2 from Mesh1)
 *  Intersection (Mesh1 & Mesh2; removing what's outside of Mesh2 from Mesh1)
 *
 */
USTRUCT()
struct FMeshBooleanDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshBooleanDataflowNode, "MeshBoolean", "Mesh|Utilities", "")

public:

	FMeshBooleanDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Boolean operation */
	UPROPERTY(EditAnywhere, Category = "Boolean");
	EMeshBooleanOperationEnum Operation = EMeshBooleanOperationEnum::Dataflow_MeshBoolean_Intersect;

	/** Whether to weld common edges at the joins of the mesh boolean */
	UPROPERTY(EditAnywhere, Category = "Boolean", AdvancedDisplay)
	bool bWeldSharedEdges = false;

	/** Whether to simplify the result along new edges */
	UPROPERTY(EditAnywhere, Category = "Simplification")
	bool bSimplifyAlongNewEdges = true;
	
	UPROPERTY(EditAnywhere, Category = "Simplification", AdvancedDisplay, DisplayName="Preserve UV Quality for Mesh", meta=(EditCondition="bSimplifyAlongNewEdges"))
	EDataflowMeshBooleanInputMeshEnum PreserveUVQualityForMesh = EDataflowMeshBooleanInputMeshEnum::First;

	/** Mesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh1;

	/** Mesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh2;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};



namespace UE::Dataflow
{
	void MeshBooleanNodes();
}

